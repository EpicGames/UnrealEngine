[Options]
UseFastModularVersioning=1
UseFastModularVersioningV2=1
UseCrashReportClientEditor=1
DebugGameHasSeparateExecutable=1
BuildUnrealHeaderTool=0
+ExcludeChanges=^Rebuild lightmaps\.$
+ExcludeChanges=^Rebuild lightmaps & texture streaming\.$
+ExcludeChanges=^Added collection '.*'\s*
+ExcludeChanges=^Added '.*' to collection '.*'\s*
+ExcludeChanges=^Added \d+ objects to collection '.*':
+ExcludeChanges=^Removed '.*' from collection '.*'\s*
+ExcludeChanges=^Removed \d+ objects from collection '.*'
+ExcludeChanges=^Changed the dynamic query of collection '.*' to '.*'
+ExcludeChanges=^Changed the parent of collection '.*'
+ExcludeChanges=^Upgraded collection '.*' \(was version \d+, now version \d+\)
+ExcludeChanges=^Collection '.*' not modified

; Sync Categories
+SyncCategory=(UniqueId="6703e989-d912-451d-93ad-b48de748d282", Name="Content", Paths="*.uasset;*.umap")
+SyncCategory=(UniqueId="6507c2fb-19dd-403a-afa3-bbf898248d5a", Name="Documentation", Paths="/Engine/Documentation/...")
+SyncCategory=(UniqueId="fd7c716e-4bad-43ae-8fae-8748ef9ee44d", Name="Platform Support: Android", Paths="/Engine/Source/ThirdParty/.../Android/...;.../Build/Android/PipelineCaches/...")
+SyncCategory=(UniqueId="176b2eb2-35f7-4e8e-b131-5f1c5f0959af", Name="Platform Support: iOS", Paths="/Engine/Source/ThirdParty/.../IOS/...;.../Build/IOS/PipelineCaches/...")
+SyncCategory=(UniqueId="f44b2d25-cbc0-4a8f-b6b3-e4a8125533dd", Name="Platform Support: Linux", Paths="/Engine/Source/ThirdParty/.../Linux/...;.../Content/Python/Lib/Linux/...")
+SyncCategory=(UniqueId="2af45231-0d75-463b-bf9f-abb3231091bb", Name="Platform Support: Mac", Paths="/Engine/Source/ThirdParty/.../Mac/...;.../Build/Mac/PipelineCaches/...;.../Content/Python/Lib/Mac/...")
+SyncCategory=(UniqueId="3788a0bc-188c-4a0d-950a-d68175f0d110", Name="Platform Support: tvOS", Paths="/Engine/Source/ThirdParty/.../TVOS/...")
+SyncCategory=(UniqueId="1144e719-fcd7-491b-b0fc-8b4c3565bf79", Name="Platform Support: Win32", Paths="/Engine/Source/ThirdParty/.../Win32/...")
+SyncCategory=(UniqueId="5206ccee-9024-4e36-8b89-f5f5a7d288d2", Name="Platform Support: Win64", Paths="/Engine/Source/ThirdParty/.../Win64/...;.../Content/Python/Lib/Win64/...")
+SyncCategory=(UniqueId="cfec942a-bb90-4f0c-accf-238ecaad9430", Name="Source Code", Paths="/Engine/Source/...")

[Build]
+Step=(UniqueId="89fe8a79-d259-4c7b-bfb4-68f7218b91c2", Description="Compile UnrealInsights", StatusText="Compile UnrealInsights Step...", EstimatedDuration="1", Type="Compile", Target="UnrealInsights", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False", OrderIndex=20)
+Step=(UniqueId="46312669-5069-428d-8d72-c2416c5a322e", Description="Launch UnrealInsights", StatusText="Running UnrealInsights", EstimatedDuration="1", Type="Other", FileName="$(BranchDir)\\Engine\\Binaries\\Win64\\UnrealInsights.exe", Arguments="", bUseLogWindow="False", bNormalSync="False", bScheduledSync="False", bShowAsTool="True", Requires="89fe8a79-d259-4c7b-bfb4-68f7218b91c2", OrderIndex=21 )
+Step=(UniqueId="bb48ca5b-5682-4432-824d-c451336a6523", Description="Compile Zen Dashboard", StatusText="Compile ZenDashboard Step...", EstimatedDuration="1", Type="Compile", Target="ZenDashboard", Platform="Win64", Configuration="Development", Arguments="", UseLogWindow="True", bNormalSync="False", bScheduledSync="False", bShowAsTool="True")
+Step=(UniqueId="586cc0d3-3914-4df9-acb6-2c02cd9d4fc6", Description="Launch Zen Dashboard", StatusText="Running Zen Dashboard", EstimatedDuration="1", Type="Other", FileName="$(BranchDir)\\Engine\\Binaries\\Win64\\ZenDashboard.exe", WorkingDir="", Arguments="", bUseLogWindow="False", bNormalSync="False", bShowAsTool="False", bScheduledSync="False", Link="Zen", LocalCompileRequires="bb48ca5b-5682-4432-824d-c451336a6523" )
+Step=(UniqueId="91c2a429-c391-49b4-92a5-4e6be71e0f00", Description="Compile SwitchboardListener", StatusText="Compiling SwitchboardListener...", EstimatedDuration="1", Type="Compile", Target="SwitchboardListener", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="71e65719-b7bb-49c5-961b-620acf1e1dad", Description="Compile SwitchboardListenerHelper", StatusText="Compiling SwitchboardListenerHelper...", EstimatedDuration="1", Type="Compile", Target="SwitchboardListenerHelper", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="5036c75b-8df0-4329-82a1-869dd2d48605", Description="Compile UnrealMultiUserServer", StatusText="Compiling UnrealMultiUserServer...", EstimatedDuration="1", Type="Compile", Target="UnrealMultiUserServer", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="274b89c3-9dc6-4465-a508-40abc4593cc2", Description="Compile UnrealMultiUserSlateServer", StatusText="Compiling UnrealMultiUserSlateServer...", EstimatedDuration="1", Type="Compile", Target="UnrealMultiUserSlateServer", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="fffcbd4b-0f14-40ca-9069-f09c1c6e5898", Description="Compile LiveLinkHub", StatusText="Compiling LiveLinkHub...", EstimatedDuration="1", Type="Compile", Target="LiveLinkHub", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="0aadd9c7-17b7-4818-a9ac-d11dc71f6ab9", Description="Launch LiveLinkHub", StatusText="Running LiveLinkHub", EstimatedDuration="1", Type="Other", FileName="$(BranchDir)\\Engine\\Binaries\\Win64\\LiveLinkHub.exe", WorkingDir="", Arguments="", bUseLogWindow="False", bNormalSync="False", bShowAsTool="True", bScheduledSync="False", Requires="fffcbd4b-0f14-40ca-9069-f09c1c6e5898" )
+Step=(UniqueId="f7f0c4c7-5551-4485-a961-f3e4fd9d138c", Description="Compile InterchangeWorker", StatusText="Compiling InterchangeWorker...", EstimatedDuration="1", Type="Compile", Target="InterchangeWorker", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False")
+Step=(UniqueId="810b581a-35fc-4550-9f1f-234494ae7c84", Description="Compile UnrealFrontend", StatusText="Compile UnrealFrontend Step...", EstimatedDuration="1", Type="Compile", Target="UnrealFrontend", Platform="Win64", Configuration="Development", Arguments="", bNormalSync="False", bScheduledSync="False", bShowAsTool="False", OrderIndex=22)
+Step=(UniqueId="8536e8df-9116-41e6-8f84-20018a31221c", Description="Launch UnrealFrontend", StatusText="Running UnrealFrontend", EstimatedDuration="1", Type="Other", FileName="$(BranchDir)\\Engine\\Binaries\\Win64\\UnrealFrontend.exe", Arguments="", bUseLogWindow="False", bNormalSync="False", bScheduledSync="False", bShowAsTool="True", Requires="810b581a-35fc-4550-9f1f-234494ae7c84", OrderIndex=23 )

[Default]
EditorTarget=UnrealEditor
+DefaultEditorArgument=(Name="-log", Enabled=false, Locked=false)
+DefaultEditorArgument=(Name="-fastload", Enabled=false, Locked=false)

; Set this to a network share which contains the SDK installers for your site
SdkInstallerDir= 

; All the required SDKs for the current version of the engine
+SdkInfo=(Category="Android", Description="NDK r27c", Browse="$(SdkInstallerDir)\\Android")
+SdkInfo=(Category="Windows", Description="Visual Studio 2022 17.14", Browse="https://visualstudio.microsoft.com/downloads/")
+SdkInfo=(Category="Windows", Description="Windows SDK 10.0.22621.0", Browse="https://developer.microsoft.com/en-us/windows/downloads/windows-sdk/")
+SdkInfo=(Category="Windows", Description="LLVM 20.1.8", Browse="https://github.com/llvm/llvm-project/releases/tag/llvmorg-20.1.8")
+SdkInfo=(Category="Linux", Description="v26_clang-20.1.8-rockylinux8", Install="$(SdkInstallerDir)\\Linux\\HostWin64\\v26_clang-20.1.8-rockylinux8.exe", Browse="$(SdkInstallerDir)\\Linux\\HostWin64")
+SdkInfo=(Category="Mac", Description="macOS 10.15.7")
+SdkInfo=(Category="Mac", Description="Xcode 26.1.1")
